<Application xmlns="https://github.com/avaloniaui"
             xmlns:x="http://schemas.microsoft.com/winfx/2006/xaml"
             xmlns:sty="using:FluentAvalonia.Styling"
             x:Class="UnrealToolboxExampleApplication.App"
             RequestedThemeVariant="Default">
    <Application.Styles>
        <sty:FluentAvaloniaTheme PreferSystemTheme="True" PreferUserAccentColor="True" />
        <StyleInclude Source="/Styles.axaml"/>
    </Application.Styles>
</Application>
