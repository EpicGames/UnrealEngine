// Copyright Epic Games, Inc. All Rights Reserved.

using Avalonia;
using Avalonia.Controls;
using Avalonia.Controls.ApplicationLifetimes;
using Avalonia.Markup.Xaml;
using Avalonia.Threading;

namespace UnrealToolboxExampleApplication;

public partial class App : Application
{
	public override void Initialize()
	{
		AvaloniaXamlLoader.Load(this);
	}

	public override void OnFrameworkInitializationCompleted()
	{
		if (ApplicationLifetime is IClassicDesktopStyleApplicationLifetime desktop)
		{
			desktop.MainWindow = new MainWindow();
			StartUpdateListener(desktop);
		}

		base.OnFrameworkInitializationCompleted();
	}

	void StartUpdateListener(IClassicDesktopStyleApplicationLifetime desktop)
	{
		if (OperatingSystem.IsWindows())
		{
			Thread thread = new Thread(() => ListenViaNamedEvent(desktop));
			thread.IsBackground = true;
			thread.Start();
		}
		else
		{
			Thread thread = new Thread(() => ListenViaMarkerFile(desktop));
			thread.IsBackground = true;
			thread.Start();
		}
	}

	void ListenViaNamedEvent(IClassicDesktopStyleApplicationLifetime desktop)
	{
		using EventWaitHandle requestEvent = new EventWaitHandle(false, EventResetMode.AutoReset, Program.UpdateRequestEventName);

		while (true)
		{
			requestEvent.WaitOne();

			ManualResetEventSlim dialogComplete = new ManualResetEventSlim(false);

			Dispatcher.UIThread.Post(async () =>
			{
				try
				{
					bool accepted = await ShowUpdateDialog(desktop);
					if (accepted)
					{
						desktop.Shutdown();
					}
					else
					{
						using EventWaitHandle declinedEvent = new EventWaitHandle(false, EventResetMode.AutoReset, Program.UpdateDeclinedEventName);
						declinedEvent.Set();
					}
				}
				finally
				{
					dialogComplete.Set();
				}
			});

			dialogComplete.Wait();
			dialogComplete.Dispose();
		}
	}

	void ListenViaMarkerFile(IClassicDesktopStyleApplicationLifetime desktop)
	{
		string requestFilePath = Program.GetRequestFilePath();

		while (true)
		{
			Thread.Sleep(1000);

			if (!File.Exists(requestFilePath))
			{
				continue;
			}

			try
			{
				File.Delete(requestFilePath);
			}
			catch
			{
				continue;
			}

			ManualResetEventSlim dialogComplete = new ManualResetEventSlim(false);

			Dispatcher.UIThread.Post(async () =>
			{
				try
				{
					bool accepted = await ShowUpdateDialog(desktop);
					if (accepted)
					{
						desktop.Shutdown();
					}
					else
					{
						try
						{
							File.WriteAllText(Program.GetDeclinedFilePath(), "declined");
						}
						catch
						{
						}
					}
				}
				finally
				{
					dialogComplete.Set();
				}
			});

			dialogComplete.Wait();
			dialogComplete.Dispose();
		}
	}

	static async Task<bool> ShowUpdateDialog(IClassicDesktopStyleApplicationLifetime desktop)
	{
		Window? mainWindow = desktop.MainWindow;
		if (mainWindow == null)
		{
			return false;
		}

		UpdateDialog dialog = new UpdateDialog();
		bool? result = await dialog.ShowDialog<bool?>(mainWindow);
		return result == true;
	}
}
