// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealToolboxExampleApplication;

static class CodeSamples
{
	public static readonly string CsprojStructure = """
		<Project Sdk="Microsoft.NET.Sdk">
		  <PropertyGroup>
		    <OutputType>WinExe</OutputType>
		    <TargetFramework>net10.0</TargetFramework>
		    <Nullable>enable</Nullable>
		    <AvaloniaUseCompiledBindingsByDefault>true</AvaloniaUseCompiledBindingsByDefault>
		  </PropertyGroup>

		  <ItemGroup>
		    <PackageReference Include="Avalonia" Version="11.3.12" />
		    <PackageReference Include="Avalonia.Desktop" Version="11.3.12" />
		    <PackageReference Include="Avalonia.Themes.Fluent" Version="11.3.12" />
		    <PackageReference Include="FluentAvaloniaUI" Version="2.5.0" />
		  </ItemGroup>
		</Project>
		""";

	public static readonly string SingleInstanceLock = """
		class SingleInstanceLock : IDisposable
		{
		    readonly string _lockFilePath;
		    FileStream? _lockFileStream;

		    public SingleInstanceLock(string appName)
		    {
		        string lockDir = Path.Combine(
		            Environment.GetFolderPath(
		                Environment.SpecialFolder.LocalApplicationData),
		            "Epic Games", appName);
		        Directory.CreateDirectory(lockDir);
		        _lockFilePath = Path.Combine(lockDir, ".lock");
		    }

		    public bool TryAcquire()
		    {
		        try
		        {
		            _lockFileStream = new FileStream(
		                _lockFilePath, FileMode.OpenOrCreate,
		                FileAccess.ReadWrite, FileShare.None);
		            return true;
		        }
		        catch (IOException) { return false; }
		    }

		    public void Dispose()
		    {
		        _lockFileStream?.Dispose();
		        _lockFileStream = null;
		    }
		}
		""";

	public static readonly string SingleInstanceUsage = """
		using SingleInstanceLock instanceLock =
		    new SingleInstanceLock("MyApp");
		if (!instanceLock.TryAcquire())
		{
		    Console.Error.WriteLine("Already running.");
		    return 1;
		}
		""";

	public static readonly string PlatformRidConditions = """
		<!-- Default to Windows for local build -->
		<ItemGroup Condition="'$(RuntimeIdentifier)' == ''
		    Or $(RuntimeIdentifier.StartsWith('win'))">
		  <Content Include="Toolbox.win.json" Link="Toolbox.json">
		    <CopyToOutputDirectory>PreserveNewest</CopyToOutputDirectory>
		    <CopyToPublishDirectory>PreserveNewest</CopyToPublishDirectory>
		  </Content>
		</ItemGroup>

		<ItemGroup Condition="$(RuntimeIdentifier.StartsWith('osx'))">
		  <Content Include="Toolbox.osx.json" Link="Toolbox.json">
		    <CopyToOutputDirectory>PreserveNewest</CopyToOutputDirectory>
		    <CopyToPublishDirectory>PreserveNewest</CopyToPublishDirectory>
		  </Content>
		</ItemGroup>

		<ItemGroup Condition="$(RuntimeIdentifier.StartsWith('linux'))">
		  <Content Include="Toolbox.linux.json" Link="Toolbox.json">
		    <CopyToOutputDirectory>PreserveNewest</CopyToOutputDirectory>
		    <CopyToPublishDirectory>PreserveNewest</CopyToPublishDirectory>
		  </Content>
		</ItemGroup>
		""";

	public static readonly string ToolboxJsonFull = """
		{
		  "installCommand": {
		    "fileName": "MyTool.exe",
		    "arguments": [ "--install" ]
		  },
		  "uninstallCommand": {
		    "fileName": "MyTool.exe",
		    "arguments": [ "--uninstall" ]
		  },
		  "runCommand": {
		    "fileName": "MyTool.exe"
		  },
		  "manualInstall": false,
		  "requiresElevation": false,
		  "hidden": false,
		  "popupMenu": {
		    "label": "My Tool",
		    "children": [
		      {
		        "label": "Open Dashboard",
		        "command": {
		          "fileName": "MyTool.exe",
		          "arguments": [ "--dashboard" ]
		        }
		      },
		      {
		        "label": "View Logs",
		        "command": {
		          "fileName": "MyTool.exe",
		          "arguments": [ "--logs" ]
		        }
		      }
		    ]
		  }
		}
		""";

	public static readonly string ToolboxJsonMinimal = """
		{
		  "installCommand": {
		    "fileName": "MyTool.exe",
		    "arguments": [ "--install" ]
		  },
		  "uninstallCommand": {
		    "fileName": "MyTool.exe",
		    "arguments": [ "--uninstall" ]
		  }
		}
		""";

	public static readonly string InstallWindows = """
		static void InstallWindows(string exePath)
		{
		    string shortcutPath = GetWindowsShortcutPath();
		    Type shellType = Type.GetTypeFromProgID("WScript.Shell")
		        ?? throw new InvalidOperationException(
		            "WScript.Shell is not available.");
		    dynamic shell = Activator.CreateInstance(shellType)!;
		    dynamic shortcut = shell.CreateShortcut(shortcutPath);
		    shortcut.TargetPath = exePath;
		    shortcut.WorkingDirectory =
		        Path.GetDirectoryName(exePath);
		    shortcut.Description = AppDisplayName;
		    shortcut.Save();

		    Marshal.ReleaseComObject(shortcut);
		    Marshal.ReleaseComObject(shell);
		}
		""";

	public static readonly string InstallMacOs = """
		static void InstallMacOS(string exePath)
		{
		    string applicationsDir = Path.Combine(
		        Environment.GetFolderPath(
		            Environment.SpecialFolder.UserProfile),
		        "Applications");
		    Directory.CreateDirectory(applicationsDir);

		    string linkPath = Path.Combine(
		        applicationsDir, ProcessName);
		    if (File.Exists(linkPath))
		        File.Delete(linkPath);

		    File.CreateSymbolicLink(linkPath, exePath);
		}
		""";

	public static readonly string InstallLinux = """
		static void InstallLinux(string exePath)
		{
		    string applicationsDir = Path.Combine(
		        Environment.GetFolderPath(
		            Environment.SpecialFolder.UserProfile),
		        ".local", "share", "applications");
		    Directory.CreateDirectory(applicationsDir);

		    string desktopFilePath = Path.Combine(
		        applicationsDir, $"{ProcessName}.desktop");
		    string content =
		        "[Desktop Entry]\n" +
		        $"Name={AppDisplayName}\n" +
		        $"Exec={exePath}\n" +
		        "Type=Application\n" +
		        "Terminal=false\n";

		    File.WriteAllText(desktopFilePath, content);
		}
		""";

	public static readonly string NamedEventUpdate = """
		// Installer sends update request
		using EventWaitHandle requestEvent =
		    new EventWaitHandle(false, EventResetMode.AutoReset,
		        "MyApp-UpdateRequest");
		requestEvent.Set();

		// Running app listens and shows dialog
		requestEvent.WaitOne();
		bool accepted = await ShowUpdateDialog();
		if (accepted)
		    desktop.Shutdown();
		else
		{
		    using EventWaitHandle declinedEvent =
		        new EventWaitHandle(false, EventResetMode.AutoReset,
		            "MyApp-UpdateDeclined");
		    declinedEvent.Set();
		}
		""";

	public static readonly string MarkerFileUpdate = """
		// Installer writes marker file
		File.WriteAllText(requestFilePath, "update");

		// Running app polls for the file
		while (true)
		{
		    Thread.Sleep(1000);
		    if (!File.Exists(requestFilePath))
		        continue;

		    File.Delete(requestFilePath);
		    bool accepted = await ShowUpdateDialog();
		    if (!accepted)
		        File.WriteAllText(declinedFilePath, "declined");
		    else
		        desktop.Shutdown();
		}
		""";

	public static readonly string ToolboxWinJson = """
		{
		  "installCommand": {
		    "fileName": "UnrealToolboxExampleApplication.exe",
		    "arguments": [ "--install" ]
		  },
		  "uninstallCommand": {
		    "fileName": "UnrealToolboxExampleApplication.exe",
		    "arguments": [ "--uninstall" ]
		  },
		  "runCommand": {
		    "fileName": "UnrealToolboxExampleApplication.exe"
		  },
		  "popupMenu": {
		    "label": "Open Developer Guide",
		    "command": {
		      "fileName": "UnrealToolboxExampleApplication.exe"
		    }
		  }
		}
		""";

	public static readonly string ToolboxUnixJson = """
		{
		  "installCommand": {
		    "fileName": "UnrealToolboxExampleApplication",
		    "arguments": [ "--install" ]
		  },
		  "uninstallCommand": {
		    "fileName": "UnrealToolboxExampleApplication",
		    "arguments": [ "--uninstall" ]
		  },
		  "runCommand": {
		    "fileName": "UnrealToolboxExampleApplication"
		  },
		  "popupMenu": {
		    "label": "Open Developer Guide",
		    "command": {
		      "fileName": "UnrealToolboxExampleApplication"
		    }
		  }
		}
		""";

	public static readonly string LocalToolFlag = """
		dotnet run --project UnrealToolbox.csproj -- \
		    -NoUpdate -LocalTool=C:\dev\MyTool\bin\publish
		""";
}
