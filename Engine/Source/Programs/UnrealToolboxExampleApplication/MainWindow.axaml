<Window xmlns="https://github.com/avaloniaui"
        xmlns:x="http://schemas.microsoft.com/winfx/2006/xaml"
        xmlns:ui="using:FluentAvalonia.UI.Controls"
        x:Class="UnrealToolboxExampleApplication.MainWindow"
        Title="Unreal Toolbox Example Application"
        Width="900"
        Height="650"
        WindowStartupLocation="CenterScreen"
        Icon="/Resources/utbea.ico">

    <Panel>
        <ui:NavigationView x:Name="_navView"
                           OpenPaneLength="260"
                           IsSettingsVisible="False"
                           PaneDisplayMode="Left"
                           PaneTitle="Developer Guide"
                           BorderBrush="Transparent">
        </ui:NavigationView>
    </Panel>

</Window>
