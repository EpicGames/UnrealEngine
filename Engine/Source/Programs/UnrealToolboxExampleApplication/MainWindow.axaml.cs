// Copyright Epic Games, Inc. All Rights Reserved.

using Avalonia.Controls;
using FluentAvalonia.UI.Controls;
using UnrealToolboxExampleApplication.Pages;

namespace UnrealToolboxExampleApplication;

public partial class MainWindow : Window
{
	readonly Dictionary<string, UserControl> _pageCache = new Dictionary<string, UserControl>();

	public MainWindow()
	{
		InitializeComponent();

		_navView.MenuItems.Add(new NavigationViewItem() { Content = "Welcome", IconSource = new SymbolIconSource() { Symbol = Symbol.Home }, Tag = "Welcome" });
		_navView.MenuItems.Add(new NavigationViewItem() { Content = "Project Setup", IconSource = new SymbolIconSource() { Symbol = Symbol.Code }, Tag = "ProjectSetup" });
		_navView.MenuItems.Add(new NavigationViewItem() { Content = "Toolbox.json Reference", IconSource = new SymbolIconSource() { Symbol = Symbol.Document }, Tag = "ToolboxJson" });
		_navView.MenuItems.Add(new NavigationViewItem() { Content = "Install & Uninstall", IconSource = new SymbolIconSource() { Symbol = Symbol.Sync }, Tag = "InstallUninstall" });
		_navView.MenuItems.Add(new NavigationViewItem() { Content = "Publishing to Horde", IconSource = new SymbolIconSource() { Symbol = Symbol.Globe }, Tag = "Publishing" });
		_navView.MenuItems.Add(new NavigationViewItem() { Content = "This Example", IconSource = new SymbolIconSource() { Symbol = Symbol.Important }, Tag = "ThisExample" });

		_navView.SelectionChanged += NavView_SelectionChanged;
		_navView.SelectedItem = _navView.MenuItems[0];
	}

	void NavView_SelectionChanged(object? sender, NavigationViewSelectionChangedEventArgs e)
	{
		if (_navView.SelectedItem is NavigationViewItem nvi)
		{
			string? tag = nvi.Tag as string;
			if (tag != null)
			{
				_navView.Content = GetOrCreatePage(tag);
			}
		}
	}

	UserControl GetOrCreatePage(string tag)
	{
		if (!_pageCache.TryGetValue(tag, out UserControl? page))
		{
			page = tag switch
			{
				"Welcome" => new WelcomePage(),
				"ProjectSetup" => new ProjectSetupPage(),
				"ToolboxJson" => new ToolboxJsonPage(),
				"InstallUninstall" => new InstallUninstallPage(),
				"Publishing" => new PublishingPage(),
				"ThisExample" => new ThisExamplePage(),
				_ => new WelcomePage()
			};
			_pageCache.Add(tag, page);
		}
		return page;
	}
}
