<UserControl xmlns="https://github.com/avaloniaui"
             xmlns:x="http://schemas.microsoft.com/winfx/2006/xaml"
             xmlns:local="using:UnrealToolboxExampleApplication"
             x:Class="UnrealToolboxExampleApplication.Pages.InstallUninstallPage">

    <ScrollViewer Padding="32 24">
        <StackPanel Spacing="4" MaxWidth="700">

            <TextBlock Classes="PageTitle" Text="Install &amp; Uninstall"/>

            <!-- Installation flow -->
            <TextBlock Classes="SectionHeader" Text="Installation Flow"/>

            <TextBlock Classes="DocBody"
                       Text="When the Toolbox installs a tool, it follows these steps:"/>

            <Border Classes="DocCard">
                <StackPanel Spacing="4">
                    <TextBlock Classes="DocBody" Text="1. Download the tool zip from the Horde server"/>
                    <TextBlock Classes="DocBody" Text="2. Extract to a versioned directory under the Toolbox data folder"/>
                    <TextBlock Classes="DocBody" Text="3. Parse Toolbox.json from the extracted root"/>
                    <TextBlock Classes="DocBody" Text="4. Run installCommand with the tool directory as the working directory"/>
                    <TextBlock Classes="DocBody" Text="5. Record the installed version in Tools.json"/>
                </StackPanel>
            </Border>

            <!-- Windows install -->
            <TextBlock Classes="SectionHeader" Text="Windows: Start Menu Shortcut"/>

            <TextBlock Classes="DocBody"
                       Text="This example creates a Start Menu shortcut using COM automation (WScript.Shell). The shortcut points to the tool executable and sets the working directory:"/>

            <Border Classes="CodeBlock">
                <SelectableTextBlock Text="{x:Static local:CodeSamples.InstallWindows}"/>
            </Border>

            <!-- macOS install -->
            <TextBlock Classes="SectionHeader" Text="macOS: Symlink in ~/Applications"/>

            <TextBlock Classes="DocBody"
                       Text="On macOS, a symbolic link is created in the user's Applications directory, providing Finder and Spotlight discoverability:"/>

            <Border Classes="CodeBlock">
                <SelectableTextBlock Text="{x:Static local:CodeSamples.InstallMacOs}"/>
            </Border>

            <!-- Linux install -->
            <TextBlock Classes="SectionHeader" Text="Linux: .desktop Entry"/>

            <TextBlock Classes="DocBody"
                       Text="On Linux, a .desktop file is written to ~/.local/share/applications, making the tool appear in application launchers:"/>

            <Border Classes="CodeBlock">
                <SelectableTextBlock Text="{x:Static local:CodeSamples.InstallLinux}"/>
            </Border>

            <!-- Graceful update -->
            <TextBlock Classes="SectionHeader" Text="Graceful Update Handshake"/>

            <TextBlock Classes="DocBody"
                       Text="When the Toolbox updates a tool, it runs the uninstallCommand first. If the tool is currently running, the uninstall process needs to ask it to shut down. This example implements a two-party handshake:"/>

            <Border Classes="DocCard">
                <StackPanel Spacing="4">
                    <TextBlock FontWeight="SemiBold" Text="Windows: Named Events"/>
                    <TextBlock Classes="DocBody"
                               Text="The uninstaller signals a named event. The running app listens on a background thread, shows an update dialog, and either shuts down (accepted) or signals a declined event."/>
                </StackPanel>
            </Border>

            <Border Classes="CodeBlock">
                <SelectableTextBlock Text="{x:Static local:CodeSamples.NamedEventUpdate}"/>
            </Border>

            <Border Classes="DocCard">
                <StackPanel Spacing="4">
                    <TextBlock FontWeight="SemiBold" Text="macOS/Linux: Marker Files"/>
                    <TextBlock Classes="DocBody"
                               Text="Named events are not available on Unix. Instead, the uninstaller writes a marker file to the temp directory. The running app polls for this file every second and responds the same way."/>
                </StackPanel>
            </Border>

            <Border Classes="CodeBlock">
                <SelectableTextBlock Text="{x:Static local:CodeSamples.MarkerFileUpdate}"/>
            </Border>

            <!-- MSI tools -->
            <TextBlock Classes="SectionHeader" Text="MSI Tools"/>

            <Border Classes="DocCard">
                <StackPanel Spacing="4">
                    <TextBlock FontWeight="SemiBold" Text="msi-product-id Metadata"/>
                    <TextBlock Classes="DocBody"
                               Text="If your tool is distributed as an MSI installer, set the msi-product-id metadata on the Horde tool deployment. The Toolbox will use this to check whether the MSI is already installed and skip redundant installs. MSI tools are not auto-updated."/>
                </StackPanel>
            </Border>

            <!-- Elevation -->
            <TextBlock Classes="SectionHeader" Text="Elevation"/>

            <Border Classes="DocCard">
                <StackPanel Spacing="4">
                    <TextBlock FontWeight="SemiBold" Text="requiresElevation"/>
                    <TextBlock Classes="DocBody"
                               Text="When set to true in Toolbox.json, the installCommand and uninstallCommand are run with elevated privileges. On Windows this triggers the UAC prompt; on macOS/Linux this uses sudo. Use this only when your install step requires writing to protected locations (e.g., Program Files, /usr/local/bin) or registering system services."/>
                </StackPanel>
            </Border>

        </StackPanel>
    </ScrollViewer>

</UserControl>
