<UserControl xmlns="https://github.com/avaloniaui"
             xmlns:x="http://schemas.microsoft.com/winfx/2006/xaml"
             xmlns:local="using:UnrealToolboxExampleApplication"
             x:Class="UnrealToolboxExampleApplication.Pages.ProjectSetupPage">

    <ScrollViewer Padding="32 24">
        <StackPanel Spacing="4" MaxWidth="700">

            <TextBlock Classes="PageTitle" Text="Project Setup"/>

            <TextBlock Classes="DocBody"
                       Text="A Toolbox tool can be any executable — .NET, native C++, Electron, etc. This guide uses a .NET Avalonia app because that is what this example application is built with, but the packaging concepts apply to any technology."/>

            <!-- .csproj structure -->
            <TextBlock Classes="SectionHeader" Text="Project File Structure"/>

            <TextBlock Classes="DocBody"
                       Text="Below is the .csproj from this example application. The key points are WinExe output type (no console window), the Avalonia and FluentAvaloniaUI package references, and compiled bindings enabled."/>

            <Border Classes="CodeBlock">
                <SelectableTextBlock Text="{x:Static local:CodeSamples.CsprojStructure}"/>
            </Border>

            <!-- Single instance lock -->
            <TextBlock Classes="SectionHeader" Text="Single-Instance Lock"/>

            <TextBlock Classes="DocBody"
                       Text="The Toolbox may launch your tool's runCommand at any time (e.g., when the user clicks it in the tray menu). If your tool is already running, a second instance should detect this and exit gracefully. A cross-platform file-lock approach works everywhere:"/>

            <Border Classes="CodeBlock">
                <SelectableTextBlock Text="{x:Static local:CodeSamples.SingleInstanceLock}"/>
            </Border>

            <TextBlock Classes="DocBody" Text="Usage in Main:"/>

            <Border Classes="CodeBlock">
                <SelectableTextBlock Text="{x:Static local:CodeSamples.SingleInstanceUsage}"/>
            </Border>

            <!-- Platform builds -->
            <TextBlock Classes="SectionHeader" Text="Platform-Specific Builds"/>

            <TextBlock Classes="DocBody"
                       Text="Horde tool deployments target specific Runtime Identifiers (RIDs) like win-x64, osx-arm64, or linux-x64. Each platform needs its own Toolbox.json because executable names differ (e.g., .exe on Windows). Use conditional ItemGroups to select the right config at publish time:"/>

            <Border Classes="CodeBlock">
                <SelectableTextBlock Text="{x:Static local:CodeSamples.PlatformRidConditions}"/>
            </Border>

            <!-- Tip card -->
            <Border Classes="DocCard" Margin="0 12 0 0">
                <StackPanel Spacing="4">
                    <TextBlock FontWeight="SemiBold" FontSize="14" Text="Tip: {ToolDir} Expansion"/>
                    <TextBlock Classes="DocBody"
                               Text="In Toolbox.json, the fileName field supports {ToolDir} as a placeholder that expands to the directory where the tool is installed. This is useful if your tool's executable is in a subdirectory: &quot;fileName&quot;: &quot;{ToolDir}/bin/mytool.exe&quot;"/>
                </StackPanel>
            </Border>

        </StackPanel>
    </ScrollViewer>

</UserControl>
