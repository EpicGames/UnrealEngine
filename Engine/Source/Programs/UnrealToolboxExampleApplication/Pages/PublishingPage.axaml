<UserControl xmlns="https://github.com/avaloniaui"
             xmlns:x="http://schemas.microsoft.com/winfx/2006/xaml"
             xmlns:local="using:UnrealToolboxExampleApplication"
             x:Class="UnrealToolboxExampleApplication.Pages.PublishingPage">

    <ScrollViewer Padding="32 24">
        <StackPanel Spacing="4" MaxWidth="700">

            <TextBlock Classes="PageTitle" Text="Publishing to Horde"/>

            <TextBlock Classes="DocBody"
                       Text="Once your tool is built and packaged with a Toolbox.json, you need to publish it to the Horde server so the Toolbox can discover and distribute it."/>

            <!-- Server-side configuration -->
            <TextBlock Classes="SectionHeader" Text="Server-Side Configuration"/>

            <Border Classes="DocCard">
                <StackPanel Spacing="4">
                    <TextBlock FontWeight="SemiBold" Text="ShowInToolbox"/>
                    <TextBlock Classes="DocBody"
                               Text="The Horde tool deployment must have the ShowInToolbox flag set to true. Without this, the Toolbox will not include the tool in its catalog even if it exists on the server."/>
                </StackPanel>
            </Border>

            <Border Classes="DocCard">
                <StackPanel Spacing="4">
                    <TextBlock FontWeight="SemiBold" Text="Platforms"/>
                    <TextBlock Classes="DocBody"
                               Text="The deployment specifies which platforms it supports using a Platforms array of .NET Runtime Identifiers. The Toolbox matches these against the local machine to determine which deployments to offer."/>
                </StackPanel>
            </Border>

            <!-- RID table -->
            <TextBlock Classes="SectionHeader" Text="Platform RID Reference"/>

            <Border Classes="DocCard">
                <StackPanel Spacing="6">
                    <TextBlock FontWeight="SemiBold" Text="Common Runtime Identifiers"/>
                    <TextBlock Classes="DocBody" Text="any — Platform-independent (e.g., scripts, JARs)"/>
                    <TextBlock Classes="DocBody" Text="win — Any Windows architecture"/>
                    <TextBlock Classes="DocBody" Text="win-x64 — Windows 64-bit Intel/AMD"/>
                    <TextBlock Classes="DocBody" Text="win-arm64 — Windows ARM64"/>
                    <TextBlock Classes="DocBody" Text="osx-x64 — macOS Intel"/>
                    <TextBlock Classes="DocBody" Text="osx-arm64 — macOS Apple Silicon"/>
                    <TextBlock Classes="DocBody" Text="linux-x64 — Linux 64-bit Intel/AMD"/>
                    <TextBlock Classes="DocBody" Text="linux-arm64 — Linux ARM64"/>
                </StackPanel>
            </Border>

            <!-- Auto-update -->
            <TextBlock Classes="SectionHeader" Text="Auto-Update Behavior"/>

            <TextBlock Classes="DocBody"
                       Text="The Toolbox polls the Horde server every 5 minutes for tool updates. When a new version is detected:"/>

            <Border Classes="DocCard">
                <StackPanel Spacing="4">
                    <TextBlock Classes="DocBody" Text="1. The new version is downloaded and extracted to a staging directory"/>
                    <TextBlock Classes="DocBody" Text="2. The old version's uninstallCommand is run (triggering the update handshake if the tool is running)"/>
                    <TextBlock Classes="DocBody" Text="3. The old version directory is deleted"/>
                    <TextBlock Classes="DocBody" Text="4. The new version's installCommand is run"/>
                    <TextBlock Classes="DocBody" Text="5. Tools.json is updated with the new version"/>
                </StackPanel>
            </Border>

            <Border Classes="DocCard">
                <StackPanel Spacing="4">
                    <TextBlock FontWeight="SemiBold" Text="Exceptions"/>
                    <TextBlock Classes="DocBody"
                               Text="Tools with manualInstall: true are not auto-updated — the user must trigger updates from the UI. Tools identified by msi-product-id are also excluded from auto-update since MSI installations are managed by Windows Installer."/>
                </StackPanel>
            </Border>

            <!-- Local development -->
            <TextBlock Classes="SectionHeader" Text="Local Development"/>

            <TextBlock Classes="DocBody"
                       Text="During development, you can test your tool locally without publishing to Horde. Run UnrealToolbox with the -LocalTool flag pointing to your tool's publish output directory:"/>

            <Border Classes="CodeBlock">
                <SelectableTextBlock Text="{x:Static local:CodeSamples.LocalToolFlag}"/>
            </Border>

            <TextBlock Classes="DocCaption"
                       Text="The local tool appears in the catalog alongside server-published tools. It is not auto-updated."/>

            <!-- Deployment checklist -->
            <TextBlock Classes="SectionHeader" Text="Deployment Checklist"/>

            <Border Classes="DocCard">
                <StackPanel Spacing="4">
                    <TextBlock FontWeight="SemiBold" Text="Before publishing, verify:"/>
                    <TextBlock Classes="DocBody" Text="1. Toolbox.json is present at the root of your publish output"/>
                    <TextBlock Classes="DocBody" Text="2. installCommand and uninstallCommand work correctly"/>
                    <TextBlock Classes="DocBody" Text="3. The tool handles single-instance correctly (second launch exits)"/>
                    <TextBlock Classes="DocBody" Text="4. The update handshake works (uninstall while running shows dialog)"/>
                    <TextBlock Classes="DocBody" Text="5. Platform RIDs in the Horde deployment match your build targets"/>
                    <TextBlock Classes="DocBody" Text="6. ShowInToolbox is enabled on the Horde tool deployment"/>
                    <TextBlock Classes="DocBody" Text="7. Test with -LocalTool before publishing to the server"/>
                </StackPanel>
            </Border>

        </StackPanel>
    </ScrollViewer>

</UserControl>
