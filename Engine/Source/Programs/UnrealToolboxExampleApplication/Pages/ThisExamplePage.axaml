<UserControl xmlns="https://github.com/avaloniaui"
             xmlns:x="http://schemas.microsoft.com/winfx/2006/xaml"
             xmlns:local="using:UnrealToolboxExampleApplication"
             x:Class="UnrealToolboxExampleApplication.Pages.ThisExamplePage">

    <ScrollViewer Padding="32 24">
        <StackPanel Spacing="4" MaxWidth="700">

            <TextBlock Classes="PageTitle" Text="This Example Application"/>

            <TextBlock Classes="DocBody"
                       Text="This application is a real, deployed Toolbox tool. Below is an inventory of its source files and an explanation of how each piece fits together."/>

            <!-- Source file inventory -->
            <TextBlock Classes="SectionHeader" Text="Source File Inventory"/>

            <Border Classes="DocCard">
                <StackPanel Spacing="6">
                    <TextBlock FontWeight="SemiBold" Text="Application Core"/>
                    <TextBlock Classes="DocBody" Text="Program.cs — Entry point, single-instance lock, install/uninstall commands, platform-specific shortcuts"/>
                    <TextBlock Classes="DocBody" Text="App.axaml / App.axaml.cs — Avalonia application setup, FluentAvalonia theme, update listener threads"/>
                    <TextBlock Classes="DocBody" Text="MainWindow.axaml / MainWindow.axaml.cs — NavigationView shell, page routing and caching"/>
                    <TextBlock Classes="DocBody" Text="UpdateDialog.axaml / UpdateDialog.axaml.cs — Modal dialog shown during the update handshake"/>
                </StackPanel>
            </Border>

            <Border Classes="DocCard">
                <StackPanel Spacing="6">
                    <TextBlock FontWeight="SemiBold" Text="Guide Pages"/>
                    <TextBlock Classes="DocBody" Text="Pages/WelcomePage.axaml — Overview and three-step summary"/>
                    <TextBlock Classes="DocBody" Text="Pages/ProjectSetupPage.axaml — .csproj structure, single-instance lock, platform builds"/>
                    <TextBlock Classes="DocBody" Text="Pages/ToolboxJsonPage.axaml — Complete Toolbox.json reference"/>
                    <TextBlock Classes="DocBody" Text="Pages/InstallUninstallPage.axaml — Install examples and update handshake"/>
                    <TextBlock Classes="DocBody" Text="Pages/PublishingPage.axaml — Horde deployment, RIDs, auto-update"/>
                    <TextBlock Classes="DocBody" Text="Pages/ThisExamplePage.axaml — This page"/>
                </StackPanel>
            </Border>

            <Border Classes="DocCard">
                <StackPanel Spacing="6">
                    <TextBlock FontWeight="SemiBold" Text="Shared Resources"/>
                    <TextBlock Classes="DocBody" Text="Styles.axaml — DocCard, CodeBlock, PageTitle, SectionHeader, DocBody, DocCaption styles"/>
                    <TextBlock Classes="DocBody" Text="CodeSamples.cs — Static code snippet strings referenced by pages via x:Static"/>
                </StackPanel>
            </Border>

            <!-- Platform config comparison -->
            <TextBlock Classes="SectionHeader" Text="Platform Configuration Comparison"/>

            <TextBlock Classes="DocBody"
                       Text="This app ships three Toolbox.json variants. The only difference is the executable name — Windows uses .exe, Unix platforms do not:"/>

            <TextBlock Classes="DocBody" FontWeight="SemiBold" Margin="0 8 0 2" Text="Toolbox.win.json"/>
            <Border Classes="CodeBlock">
                <SelectableTextBlock Text="{x:Static local:CodeSamples.ToolboxWinJson}"/>
            </Border>

            <TextBlock Classes="DocBody" FontWeight="SemiBold" Margin="0 8 0 2" Text="Toolbox.osx.json / Toolbox.linux.json"/>
            <Border Classes="CodeBlock">
                <SelectableTextBlock Text="{x:Static local:CodeSamples.ToolboxUnixJson}"/>
            </Border>

            <!-- Update dialog -->
            <TextBlock Classes="SectionHeader" Text="Update Dialog"/>

            <TextBlock Classes="DocBody"
                       Text="When the Toolbox needs to update this tool while it is running, it triggers the update handshake (named event on Windows, marker file on macOS/Linux). The App.axaml.cs background listener thread detects the signal and posts to the UI thread to show UpdateDialog — a simple Yes/No modal."/>

            <Border Classes="DocCard">
                <StackPanel Spacing="4">
                    <TextBlock Classes="DocBody" Text="If the user clicks Yes, the application shuts down so the Toolbox can replace the files and run the new version's installCommand."/>
                    <TextBlock Classes="DocBody" Text="If the user clicks No, the app signals back (via declined event or marker file) and the update is deferred until the user closes the app or accepts a future prompt."/>
                </StackPanel>
            </Border>

            <!-- Interactive suggestions -->
            <TextBlock Classes="SectionHeader" Text="Try It"/>

            <Border Classes="DocCard">
                <StackPanel Spacing="6">
                    <TextBlock FontWeight="SemiBold" Text="Interactive Suggestions"/>
                    <TextBlock Classes="DocBody"
                               Text="Open the Unreal Toolbox settings window — this tool should appear in the tool catalog list."/>
                    <TextBlock Classes="DocBody"
                               Text="Look at the system tray — if this tool defines popupMenu items, they appear in the Toolbox tray context menu."/>
                    <TextBlock Classes="DocBody"
                               Text="Try running a second instance of this application — it should detect the lock file and exit immediately with an error message."/>
                    <TextBlock Classes="DocBody"
                               Text="Examine the source files side-by-side with this guide to see how each concept is implemented in practice."/>
                </StackPanel>
            </Border>

        </StackPanel>
    </ScrollViewer>

</UserControl>
