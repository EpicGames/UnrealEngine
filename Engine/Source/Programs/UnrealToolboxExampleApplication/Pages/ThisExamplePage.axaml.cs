// Copyright Epic Games, Inc. All Rights Reserved.

using Avalonia.Controls;

namespace UnrealToolboxExampleApplication.Pages;

public partial class ThisExamplePage : UserControl
{
	public ThisExamplePage()
	{
		InitializeComponent();
	}
}
