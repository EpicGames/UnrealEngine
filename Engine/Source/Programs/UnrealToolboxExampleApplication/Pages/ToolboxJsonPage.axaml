<UserControl xmlns="https://github.com/avaloniaui"
             xmlns:x="http://schemas.microsoft.com/winfx/2006/xaml"
             xmlns:local="using:UnrealToolboxExampleApplication"
             x:Class="UnrealToolboxExampleApplication.Pages.ToolboxJsonPage">

    <ScrollViewer Padding="32 24">
        <StackPanel Spacing="4" MaxWidth="700">

            <TextBlock Classes="PageTitle" Text="Toolbox.json Reference"/>

            <TextBlock Classes="DocBody"
                       Text="Every Toolbox tool must include a Toolbox.json file at the root of its deployment zip. This file tells the Toolbox how to install, uninstall, and run the tool, and optionally configures tray menu items and other behavior."/>

            <!-- Commands -->
            <TextBlock Classes="SectionHeader" Text="Commands"/>

            <TextBlock Classes="DocBody"
                       Text="Commands use the ToolCommand schema: an object with a fileName string and an optional arguments array of strings."/>

            <Border Classes="DocCard">
                <StackPanel Spacing="4">
                    <TextBlock FontWeight="SemiBold" Text="installCommand"/>
                    <TextBlock Classes="DocBody"
                               Text="Runs after the tool is downloaded and extracted. Use this to create shortcuts, symlinks, register services, or perform any first-time setup. Receives the extracted directory as the working directory."/>
                </StackPanel>
            </Border>

            <Border Classes="DocCard">
                <StackPanel Spacing="4">
                    <TextBlock FontWeight="SemiBold" Text="uninstallCommand"/>
                    <TextBlock Classes="DocBody"
                               Text="Runs before the tool directory is deleted. Use this to remove shortcuts, stop services, and clean up. The tool directory is still present when this runs."/>
                </StackPanel>
            </Border>

            <Border Classes="DocCard">
                <StackPanel Spacing="4">
                    <TextBlock FontWeight="SemiBold" Text="runCommand"/>
                    <TextBlock Classes="DocBody"
                               Text="Optional. Launched when the user clicks the tool in the tray menu or opens it from the Toolbox settings. If not set, the tool has no direct launch action."/>
                </StackPanel>
            </Border>

            <!-- Options -->
            <TextBlock Classes="SectionHeader" Text="Options"/>

            <Border Classes="DocCard">
                <StackPanel Spacing="4">
                    <TextBlock FontWeight="SemiBold" Text="manualInstall"/>
                    <TextBlock Classes="DocBody"
                               Text="Default: false. When true, the Toolbox downloads the tool but does not run the installCommand automatically. The user must trigger installation manually from the settings UI. Auto-update is also disabled."/>
                </StackPanel>
            </Border>

            <Border Classes="DocCard">
                <StackPanel Spacing="4">
                    <TextBlock FontWeight="SemiBold" Text="requiresElevation"/>
                    <TextBlock Classes="DocBody"
                               Text="Default: false. When true, the installCommand and uninstallCommand are run with administrator/root privileges. On Windows, this triggers the UAC prompt."/>
                </StackPanel>
            </Border>

            <Border Classes="DocCard">
                <StackPanel Spacing="4">
                    <TextBlock FontWeight="SemiBold" Text="hidden"/>
                    <TextBlock Classes="DocBody"
                               Text="Default: false. When true, the tool does not appear in the Toolbox settings UI. It is still installed and updated, but invisible to the user. Useful for background services."/>
                </StackPanel>
            </Border>

            <!-- Tray Menu -->
            <TextBlock Classes="SectionHeader" Text="Tray Menu (popupMenu)"/>

            <TextBlock Classes="DocBody"
                       Text="The popupMenu object is a single ToolMenuItem that appears in the Toolbox system tray context menu. For multiple items, use children to create a submenu. The ToolMenuItem schema:"/>

            <Border Classes="DocCard">
                <StackPanel Spacing="4">
                    <TextBlock FontWeight="SemiBold" Text="ToolMenuItem Schema"/>
                    <TextBlock Classes="DocBody" Text="label — Display text for the menu item"/>
                    <TextBlock Classes="DocBody" Text="command — A ToolCommand to run when clicked (fileName + arguments)"/>
                    <TextBlock Classes="DocBody" Text="children — Nested array of ToolMenuItem for submenus"/>
                    <TextBlock Classes="DocCaption" Margin="0 4 0 0"
                               Text="Each item should have either a command or children, not both."/>
                </StackPanel>
            </Border>

            <!-- Platform files -->
            <TextBlock Classes="SectionHeader" Text="Platform-Specific Files"/>

            <TextBlock Classes="DocBody"
                       Text="You can provide separate config files for each platform. At build/publish time, the appropriate variant is copied as Toolbox.json:"/>

            <Border Classes="DocCard">
                <StackPanel Spacing="4">
                    <TextBlock Classes="DocBody" Text="Toolbox.win.json — Windows (executables use .exe suffix)"/>
                    <TextBlock Classes="DocBody" Text="Toolbox.osx.json — macOS (no extension)"/>
                    <TextBlock Classes="DocBody" Text="Toolbox.linux.json — Linux (no extension)"/>
                </StackPanel>
            </Border>

            <!-- Full example -->
            <TextBlock Classes="SectionHeader" Text="Full Annotated Example"/>

            <TextBlock Classes="DocBody"
                       Text="A complete Toolbox.json showing all fields. In practice, most tools only need installCommand, uninstallCommand, and optionally runCommand."/>

            <Border Classes="CodeBlock">
                <SelectableTextBlock Text="{x:Static local:CodeSamples.ToolboxJsonFull}"/>
            </Border>

            <TextBlock Classes="DocBody" Margin="0 8 0 0" Text="Minimal example (install and uninstall only):"/>

            <Border Classes="CodeBlock">
                <SelectableTextBlock Text="{x:Static local:CodeSamples.ToolboxJsonMinimal}"/>
            </Border>

        </StackPanel>
    </ScrollViewer>

</UserControl>
