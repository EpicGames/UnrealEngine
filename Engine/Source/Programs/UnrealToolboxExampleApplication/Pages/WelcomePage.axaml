<UserControl xmlns="https://github.com/avaloniaui"
             xmlns:x="http://schemas.microsoft.com/winfx/2006/xaml"
             x:Class="UnrealToolboxExampleApplication.Pages.WelcomePage">

    <ScrollViewer Padding="32 24">
        <StackPanel Spacing="4" MaxWidth="700">

            <Border CornerRadius="8" ClipToBounds="True" Margin="0 0 0 16">
                <Image Source="/Resources/utbea_banner.png" Stretch="Uniform" MaxHeight="220"/>
            </Border>

            <TextBlock Classes="PageTitle" Text="Welcome to the UnrealToolbox Developer Guide"/>

            <TextBlock Classes="DocBody"
                       Text="UnrealToolbox is a Windows/macOS/Linux system tray application that manages Horde build tools and artifacts. Tools published to the Horde server are automatically discovered, downloaded, installed, and kept up to date by the Toolbox."/>

            <TextBlock Classes="DocBody" Margin="0 8 0 0"
                       Text="This guide walks you through publishing your own tool so it appears in the Toolbox catalog. Each page covers one step of the process."/>

            <TextBlock Classes="SectionHeader" Text="The Three Steps"/>

            <Border Classes="DocCard">
                <StackPanel Spacing="4">
                    <TextBlock FontWeight="SemiBold" FontSize="14" Text="1. Package Your Tool"/>
                    <TextBlock Classes="DocBody"
                               Text="Create a .NET (or native) application, add a single-instance lock, and set up platform-specific builds. The Project Setup page covers this in detail."/>
                </StackPanel>
            </Border>

            <Border Classes="DocCard">
                <StackPanel Spacing="4">
                    <TextBlock FontWeight="SemiBold" FontSize="14" Text="2. Configure Toolbox.json"/>
                    <TextBlock Classes="DocBody"
                               Text="Add a Toolbox.json file to the root of your tool's zip archive. This tells the Toolbox how to install, uninstall, and run your tool, and optionally adds tray menu items."/>
                </StackPanel>
            </Border>

            <Border Classes="DocCard">
                <StackPanel Spacing="4">
                    <TextBlock FontWeight="SemiBold" FontSize="14" Text="3. Publish to Horde"/>
                    <TextBlock Classes="DocBody"
                               Text="Upload the tool to the Horde server with the ShowInToolbox flag enabled. The Toolbox will discover it within 5 minutes and offer it to users."/>
                </StackPanel>
            </Border>

            <Border Classes="DocCard" Margin="0 16 0 0">
                <StackPanel Spacing="4">
                    <TextBlock FontWeight="SemiBold" FontSize="14" Text="This application is itself a deployed Toolbox tool"/>
                    <TextBlock Classes="DocCaption"
                               Text="Every code sample in this guide is drawn from the source code of this example application. You can inspect the source alongside these pages to see a real, working implementation of each concept."/>
                </StackPanel>
            </Border>

        </StackPanel>
    </ScrollViewer>

</UserControl>
