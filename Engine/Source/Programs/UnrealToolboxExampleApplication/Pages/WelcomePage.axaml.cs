// Copyright Epic Games, Inc. All Rights Reserved.

using Avalonia.Controls;

namespace UnrealToolboxExampleApplication.Pages;

public partial class WelcomePage : UserControl
{
	public WelcomePage()
	{
		InitializeComponent();
	}
}
