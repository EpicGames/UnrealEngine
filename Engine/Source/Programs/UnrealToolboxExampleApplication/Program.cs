// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics;
using System.Diagnostics.CodeAnalysis;
using System.Reflection;
using System.Runtime.InteropServices;
using System.Runtime.Versioning;
using Avalonia;

namespace UnrealToolboxExampleApplication;

static class Program
{
	public const string UpdateRequestEventName = "UnrealToolboxExampleApp-UpdateRequest";
	public const string UpdateDeclinedEventName = "UnrealToolboxExampleApp-UpdateDeclined";

	const string ProcessName = "UnrealToolboxExampleApplication";
	const string AppDisplayName = "Unreal Toolbox Example Application";

	[STAThread]
	public static int Main(string[] args)
	{
		if (args.Any(x => x.Equals("--install", StringComparison.OrdinalIgnoreCase)))
		{
			return Install();
		}
		if (args.Any(x => x.Equals("--uninstall", StringComparison.OrdinalIgnoreCase)))
		{
			return Uninstall();
		}

		using SingleInstanceLock instanceLock = new SingleInstanceLock("UnrealToolboxExampleApplication");
		if (!instanceLock.TryAcquire())
		{
			Console.Error.WriteLine("Another instance is already running.");
			return 1;
		}

		BuildAvaloniaApp().StartWithClassicDesktopLifetime(args);
		return 0;
	}

	public static AppBuilder BuildAvaloniaApp()
	{
		return AppBuilder.Configure<App>()
			.UsePlatformDetect()
			.LogToTrace();
	}

	static int Install()
	{
		try
		{
			string exePath = Environment.ProcessPath ?? throw new InvalidOperationException("Cannot determine process path.");

			if (OperatingSystem.IsWindows())
			{
				InstallWindows(exePath);
			}
			else if (OperatingSystem.IsMacOS())
			{
				InstallMacOS(exePath);
			}
			else if (OperatingSystem.IsLinux())
			{
				InstallLinux(exePath);
			}

			return 0;
		}
		catch (Exception ex)
		{
			Console.Error.WriteLine($"Install failed: {ex.Message}");
			return 1;
		}
	}

	static int Uninstall()
	{
		try
		{
			if (!RequestRunningInstanceShutdown())
			{
				return 1;
			}

			if (OperatingSystem.IsWindows())
			{
				UninstallWindows();
			}
			else if (OperatingSystem.IsMacOS())
			{
				UninstallMacOS();
			}
			else if (OperatingSystem.IsLinux())
			{
				UninstallLinux();
			}

			return 0;
		}
		catch (Exception ex)
		{
			Console.Error.WriteLine($"Uninstall failed: {ex.Message}");
			return 1;
		}
	}

	#region Running instance shutdown

	static bool RequestRunningInstanceShutdown()
	{
		Process[] instances = Process.GetProcessesByName(ProcessName)
			.Where(p => p.Id != Environment.ProcessId)
			.ToArray();

		if (instances.Length == 0)
		{
			return true;
		}

		if (OperatingSystem.IsWindows())
		{
			return RequestShutdownViaNamedEvent(instances);
		}
		else
		{
			return RequestShutdownViaMarkerFile(instances);
		}
	}

	static bool RequestShutdownViaNamedEvent(Process[] instances)
	{
		using EventWaitHandle requestEvent = new EventWaitHandle(false, EventResetMode.AutoReset, UpdateRequestEventName);
		using EventWaitHandle declinedEvent = new EventWaitHandle(false, EventResetMode.AutoReset, UpdateDeclinedEventName);

		Console.WriteLine("Requesting running instance to close for update...");
		requestEvent.Set();

		using ManualResetEvent allExitedEvent = new ManualResetEvent(false);
		Thread waitThread = new Thread(() =>
		{
			foreach (Process p in instances)
			{
				try
				{
					p.WaitForExit();
				}
				catch
				{
				}
				finally
				{
					p.Dispose();
				}
			}
			allExitedEvent.Set();
		});
		waitThread.IsBackground = true;
		waitThread.Start();

		int index = WaitHandle.WaitAny(new WaitHandle[] { allExitedEvent, declinedEvent }, TimeSpan.FromSeconds(60));
		if (index != 0)
		{
			string reason = index == 1 ? "User declined the update." : "Timed out waiting for user response.";
			Console.Error.WriteLine($"Uninstall aborted: {reason}");
			return false;
		}

		Console.WriteLine("Running instance closed.");
		return true;
	}

	static bool RequestShutdownViaMarkerFile(Process[] instances)
	{
		// On Mac/Linux, use a temp file to ask the running instance to show the update dialog.
		// The running app polls for the request file. If the user declines, it writes a declined file.
		string requestFilePath = GetRequestFilePath();
		string declinedFilePath = GetDeclinedFilePath();

		if (File.Exists(declinedFilePath))
		{
			File.Delete(declinedFilePath);
		}

		Console.WriteLine("Writing update request marker file...");
		File.WriteAllText(requestFilePath, "update");

		// Poll for process exit or declined file
		DateTime deadline = DateTime.UtcNow.AddSeconds(60);
		while (DateTime.UtcNow < deadline)
		{
			if (File.Exists(declinedFilePath))
			{
				try
				{
					File.Delete(declinedFilePath);
				}
				catch
				{
				}
				Console.Error.WriteLine("Uninstall aborted: User declined the update.");
				foreach (Process p in instances)
				{
					p.Dispose();
				}
				return false;
			}

			bool allExited = true;
			foreach (Process p in instances)
			{
				try
				{
					if (!p.HasExited)
					{
						allExited = false;
						break;
					}
				}
				catch
				{
				}
			}

			if (allExited)
			{
				foreach (Process p in instances)
				{
					p.Dispose();
				}
				Console.WriteLine("Running instance closed.");
				return true;
			}

			Thread.Sleep(500);
		}

		// Clean up on timeout
		try
		{
			File.Delete(requestFilePath);
		}
		catch
		{
		}
		foreach (Process p in instances)
		{
			p.Dispose();
		}
		Console.Error.WriteLine("Uninstall aborted: Timed out waiting for user response.");
		return false;
	}

	public static string GetRequestFilePath()
	{
		return Path.Combine(Path.GetTempPath(), "UnrealToolboxExampleApp-UpdateRequest");
	}

	public static string GetDeclinedFilePath()
	{
		return Path.Combine(Path.GetTempPath(), "UnrealToolboxExampleApp-UpdateDeclined");
	}

	#endregion

	#region Windows

	[SupportedOSPlatform("windows")]
	[UnconditionalSuppressMessage("Trimming", "IL2057", Justification = "COM type WScript.Shell is resolved at runtime by the OS, not subject to trimming.")]
	[UnconditionalSuppressMessage("Trimming", "IL2072", Justification = "COM type WScript.Shell is resolved at runtime by the OS, not subject to trimming.")]
	[UnconditionalSuppressMessage("Trimming", "IL2075", Justification = "COM interop members are provided by the OS, not trimmed from the assembly.")]
	static void InstallWindows(string exePath)
	{
		string shortcutPath = GetWindowsShortcutPath();

		Type shellType = Type.GetTypeFromProgID("WScript.Shell") ?? throw new InvalidOperationException("WScript.Shell is not available.");
		object shell = Activator.CreateInstance(shellType)!;
		object shortcut = shellType.InvokeMember("CreateShortcut", BindingFlags.InvokeMethod, null, shell, new object[] { shortcutPath })!;
		Type shortcutType = shortcut.GetType();
		shortcutType.InvokeMember("TargetPath", BindingFlags.SetProperty, null, shortcut, new object[] { exePath });
		shortcutType.InvokeMember("WorkingDirectory", BindingFlags.SetProperty, null, shortcut, new object[] { Path.GetDirectoryName(exePath)! });
		shortcutType.InvokeMember("Description", BindingFlags.SetProperty, null, shortcut, new object[] { AppDisplayName });
		shortcutType.InvokeMember("Save", BindingFlags.InvokeMethod, null, shortcut, null);

		Marshal.ReleaseComObject(shortcut);
		Marshal.ReleaseComObject(shell);

		Console.WriteLine($"Shortcut created: {shortcutPath}");
	}

	[SupportedOSPlatform("windows")]
	static void UninstallWindows()
	{
		string shortcutPath = GetWindowsShortcutPath();
		if (File.Exists(shortcutPath))
		{
			File.Delete(shortcutPath);
			Console.WriteLine($"Shortcut removed: {shortcutPath}");
		}
	}

	static string GetWindowsShortcutPath()
	{
		string programsFolder = Environment.GetFolderPath(Environment.SpecialFolder.Programs);
		return Path.Combine(programsFolder, $"{AppDisplayName}.lnk");
	}

	#endregion

	#region macOS

	static void InstallMacOS(string exePath)
	{
		string applicationsDir = Path.Combine(
			Environment.GetFolderPath(Environment.SpecialFolder.UserProfile),
			"Applications");
		Directory.CreateDirectory(applicationsDir);

		string linkPath = Path.Combine(applicationsDir, ProcessName);
		if (File.Exists(linkPath))
		{
			File.Delete(linkPath);
		}

		File.CreateSymbolicLink(linkPath, exePath);
		Console.WriteLine($"Symlink created: {linkPath} -> {exePath}");
	}

	static void UninstallMacOS()
	{
		string linkPath = Path.Combine(
			Environment.GetFolderPath(Environment.SpecialFolder.UserProfile),
			"Applications",
			ProcessName);

		if (Path.Exists(linkPath))
		{
			File.Delete(linkPath);
			Console.WriteLine($"Symlink removed: {linkPath}");
		}
	}

	#endregion

	#region Linux

	static void InstallLinux(string exePath)
	{
		string applicationsDir = Path.Combine(
			Environment.GetFolderPath(Environment.SpecialFolder.UserProfile),
			".local", "share", "applications");
		Directory.CreateDirectory(applicationsDir);

		string desktopFilePath = Path.Combine(applicationsDir, $"{ProcessName}.desktop");
		string content =
			"[Desktop Entry]\n" +
			$"Name={AppDisplayName}\n" +
			$"Exec={exePath}\n" +
			"Type=Application\n" +
			"Terminal=false\n" +
			$"Comment={AppDisplayName}\n";

		File.WriteAllText(desktopFilePath, content);
		Console.WriteLine($"Desktop entry created: {desktopFilePath}");
	}

	static void UninstallLinux()
	{
		string desktopFilePath = Path.Combine(
			Environment.GetFolderPath(Environment.SpecialFolder.UserProfile),
			".local", "share", "applications",
			$"{ProcessName}.desktop");

		if (File.Exists(desktopFilePath))
		{
			File.Delete(desktopFilePath);
			Console.WriteLine($"Desktop entry removed: {desktopFilePath}");
		}
	}

	#endregion
}

class SingleInstanceLock : IDisposable
{
	readonly string _lockFilePath;
	FileStream? _lockFileStream;

	public SingleInstanceLock(string appName)
	{
		string lockDir = Path.Combine(Environment.GetFolderPath(Environment.SpecialFolder.LocalApplicationData), "Epic Games", appName);
		Directory.CreateDirectory(lockDir);
		_lockFilePath = Path.Combine(lockDir, ".lock");
	}

	public bool TryAcquire()
	{
		try
		{
			_lockFileStream = new FileStream(_lockFilePath, FileMode.OpenOrCreate, FileAccess.ReadWrite, FileShare.None);
			return true;
		}
		catch (IOException)
		{
			return false;
		}
	}

	public void Dispose()
	{
		_lockFileStream?.Dispose();
		_lockFileStream = null;
	}
}
