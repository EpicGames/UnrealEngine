<Styles xmlns="https://github.com/avaloniaui"
		xmlns:x="http://schemas.microsoft.com/winfx/2006/xaml">

	<Style Selector="Border.DocCard">
		<Setter Property="BorderBrush" Value="{DynamicResource CardStrokeColorDefaultBrush}"/>
		<Setter Property="BorderThickness" Value="1"/>
		<Setter Property="Background" Value="{DynamicResource CardBackgroundFillColorDefaultBrush}"/>
		<Setter Property="TextElement.Foreground" Value="{DynamicResource TextFillColorPrimaryBrush}"/>
		<Setter Property="CornerRadius" Value="{DynamicResource ControlCornerRadius}"/>
		<Setter Property="BackgroundSizing" Value="InnerBorderEdge"/>
		<Setter Property="Padding" Value="16 14"/>
		<Setter Property="Margin" Value="0 4"/>
	</Style>

	<Style Selector="Border.CodeBlock">
		<Setter Property="Background" Value="#1E1E1E"/>
		<Setter Property="CornerRadius" Value="4"/>
		<Setter Property="Padding" Value="16 12"/>
		<Setter Property="Margin" Value="0 6"/>
	</Style>
	<Style Selector="Border.CodeBlock SelectableTextBlock">
		<Setter Property="FontFamily" Value="Cascadia Mono, Consolas, Menlo, monospace"/>
		<Setter Property="FontSize" Value="12"/>
		<Setter Property="Foreground" Value="#D4D4D4"/>
		<Setter Property="TextWrapping" Value="NoWrap"/>
	</Style>

	<Style Selector="TextBlock.PageTitle">
		<Setter Property="Theme" Value="{DynamicResource SubtitleTextBlockStyle}"/>
		<Setter Property="Margin" Value="0 0 0 12"/>
	</Style>

	<Style Selector="TextBlock.SectionHeader">
		<Setter Property="Theme" Value="{DynamicResource BodyStrongTextBlockStyle}"/>
		<Setter Property="Margin" Value="0 20 0 8"/>
	</Style>

	<Style Selector="TextBlock.DocBody">
		<Setter Property="FontSize" Value="13"/>
		<Setter Property="TextWrapping" Value="Wrap"/>
		<Setter Property="Margin" Value="0 2"/>
	</Style>

	<Style Selector="TextBlock.DocCaption">
		<Setter Property="Theme" Value="{DynamicResource CaptionTextBlockStyle}"/>
		<Setter Property="FontWeight" Value="Light"/>
		<Setter Property="Margin" Value="0 2"/>
	</Style>

</Styles>
