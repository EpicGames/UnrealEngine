{
  "installCommand": {
    "fileName": "UnrealToolboxExampleApplication.exe",
    "arguments": [ "--install" ]
  },
  "uninstallCommand": {
    "fileName": "UnrealToolboxExampleApplication.exe",
    "arguments": [ "--uninstall" ]
  },
  "runCommand": {
    "fileName": "UnrealToolboxExampleApplication.exe"
  },
  "popupMenu": {
    "label": "Open Developer Guide",
    "command": {
      "fileName": "UnrealToolboxExampleApplication.exe"
    }
  }
}
