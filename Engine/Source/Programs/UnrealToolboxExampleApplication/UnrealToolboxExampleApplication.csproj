<Project Sdk="Microsoft.NET.Sdk">

  <PropertyGroup>
    <OutputType>WinExe</OutputType>
    <TargetFramework>net10.0</TargetFramework>
    <Nullable>enable</Nullable>
    <ImplicitUsings>enable</ImplicitUsings>
    <AvaloniaUseCompiledBindingsByDefault>true</AvaloniaUseCompiledBindingsByDefault>
    <ApplicationIcon>Resources/utbea.ico</ApplicationIcon>
    <SelfContained>true</SelfContained>
    <PublishSingleFile>true</PublishSingleFile>
    <PublishTrimmed>true</PublishTrimmed>
    <PublishReadyToRun>true</PublishReadyToRun>
  </PropertyGroup>

  <ItemGroup>
    <AvaloniaResource Include="Resources\**" />
  </ItemGroup>

  <ItemGroup>
    <PackageReference Include="Avalonia" Version="11.3.12" />
    <PackageReference Include="Avalonia.Desktop" Version="11.3.12" />
    <PackageReference Include="Avalonia.Themes.Fluent" Version="11.3.12" />
    <PackageReference Include="Avalonia.Fonts.Inter" Version="11.3.12" />
    <PackageReference Include="FluentAvaloniaUI" Version="2.5.0" />
  </ItemGroup>

  <!-- Platform-specific Toolbox.json: copy the matching variant as Toolbox.json at publish time -->
  <ItemGroup>
    <None Include="Toolbox.win.json" CopyToPublishDirectory="Never" />
    <None Include="Toolbox.osx.json" CopyToPublishDirectory="Never" />
    <None Include="Toolbox.linux.json" CopyToPublishDirectory="Never" />
  </ItemGroup>

  <!-- Default to Windows for local build output (dotnet build without -r) -->
  <ItemGroup Condition="'$(RuntimeIdentifier)' == '' Or $(RuntimeIdentifier.StartsWith('win'))">
    <Content Include="Toolbox.win.json" Link="Toolbox.json">
      <CopyToOutputDirectory>PreserveNewest</CopyToOutputDirectory>
      <CopyToPublishDirectory>PreserveNewest</CopyToPublishDirectory>
    </Content>
  </ItemGroup>

  <ItemGroup Condition="$(RuntimeIdentifier.StartsWith('osx'))">
    <Content Include="Toolbox.osx.json" Link="Toolbox.json">
      <CopyToOutputDirectory>PreserveNewest</CopyToOutputDirectory>
      <CopyToPublishDirectory>PreserveNewest</CopyToPublishDirectory>
    </Content>
  </ItemGroup>

  <ItemGroup Condition="$(RuntimeIdentifier.StartsWith('linux'))">
    <Content Include="Toolbox.linux.json" Link="Toolbox.json">
      <CopyToOutputDirectory>PreserveNewest</CopyToOutputDirectory>
      <CopyToPublishDirectory>PreserveNewest</CopyToPublishDirectory>
    </Content>
  </ItemGroup>

</Project>
