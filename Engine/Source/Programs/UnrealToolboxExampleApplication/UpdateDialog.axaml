<Window xmlns="https://github.com/avaloniaui"
        xmlns:x="http://schemas.microsoft.com/winfx/2006/xaml"
        x:Class="UnrealToolboxExampleApplication.UpdateDialog"
        Title="Update Available"
        Width="400"
        Height="180"
        WindowStartupLocation="CenterOwner"
        CanResize="False"
        ShowInTaskbar="False">
    <StackPanel Margin="24" Spacing="16">
        <TextBlock Text="An update is available. Close the application to continue?"
                   TextWrapping="Wrap"
                   FontSize="14" />
        <StackPanel Orientation="Horizontal" HorizontalAlignment="Right" Spacing="8">
            <Button Content="Yes" Width="80" Click="OnYesClicked" />
            <Button Content="No" Width="80" Click="OnNoClicked" />
        </StackPanel>
    </StackPanel>
</Window>
