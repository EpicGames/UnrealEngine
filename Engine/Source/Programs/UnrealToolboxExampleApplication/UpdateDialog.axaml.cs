// Copyright Epic Games, Inc. All Rights Reserved.

using Avalonia.Controls;
using Avalonia.Interactivity;

namespace UnrealToolboxExampleApplication;

public partial class UpdateDialog : Window
{
	public UpdateDialog()
	{
		InitializeComponent();
	}

	void OnYesClicked(object? sender, RoutedEventArgs e)
	{
		Close(true);
	}

	void OnNoClicked(object? sender, RoutedEventArgs e)
	{
		Close(false);
	}
}
